{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Motion Design For nDisplay",
	"Description": "Motion Design Synchronization extensions for nDisplay clustering",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux"
	],
	"Modules": [
		{
			"Name": "AvalancheDisplayCluster",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux"
			],
			"TargetDenyList":  [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "Avalanche",
			"Enabled": true
		},
		{
			"Name": "nDisplay",
			"Enabled": true
		}
	]
}