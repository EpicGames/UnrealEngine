// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDisplayClusterModule.h"

#include "AvaDisplayClusterLog.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "SynchronizedEvents/AvaDisplayClusterSynchronizedEventsFeature.h"

DEFINE_LOG_CATEGORY(LogAvaDisplayCluster);

void FAvaDisplayClusterModule::StartupModule()
{
	SyncEventsFeature = MakeUnique<FAvaDisplayClusterSynchronizedEventsFeature>();
	IModularFeatures::Get().RegisterModularFeature(IAvaMediaSynchronizedEventsFeature::GetModularFeatureName(), SyncEventsFeature.Get());
}

void FAvaDisplayClusterModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IAvaMediaSynchronizedEventsFeature::GetModularFeatureName(), SyncEventsFeature.Get());
	SyncEventsFeature.Reset();
}

IMPLEMENT_MODULE(FAvaDisplayClusterModule, AvalancheDisplayCluster)
