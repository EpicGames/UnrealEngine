// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"

class FAvaDisplayClusterSynchronizedEventsFeature;

class FAvaDisplayClusterModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	TUniquePtr<FAvaDisplayClusterSynchronizedEventsFeature> SyncEventsFeature;
};
