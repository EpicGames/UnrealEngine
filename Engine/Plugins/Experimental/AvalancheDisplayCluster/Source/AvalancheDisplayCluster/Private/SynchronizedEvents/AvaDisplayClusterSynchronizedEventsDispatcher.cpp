// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDisplayClusterSynchronizedEventsDispatcher.h"

#include "AvaDisplayClusterSyncEventsLog.h"
#include "Cluster/DisplayClusterClusterEvent.h"
#include "Cluster/IDisplayClusterClusterManager.h"
#include "HAL/IConsoleManager.h"
#include "IDisplayCluster.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Serialization/ArrayWriter.h"

namespace UE::AvaDisplayCluster::Private
{
	// Allow events to be dispatched as early as possible (at most 1 frame earlier).
	TAutoConsoleVariable<bool> CVarSyncEarlyDispatch(
		TEXT("AvaDisplayCluster.Sync.EarlyDispatch")
		, false
		, TEXT("If true, will dispatch events as soon as ready. if false, ready events are all batched on the next tick."), ECVF_Cheat);

	static TAutoConsoleVariable<float> CVarSyncDispatchTimeout(
		TEXT("AvaDisplayCluster.Sync.DispatchTimeout"), 5000.0f,
		TEXT("Delay after witch the event is dispatched even if not signaled by all the other nodes. Units: milliseconds"),
		ECVF_Default);

	static TAutoConsoleVariable<float> CVarSyncRepeatTimeout(
		TEXT("AvaDisplayCluster.Sync.RepeatTimeout"), 200.0f,
		TEXT("Delay after witch the event is re-emitted in case cluster events don't get through for some reason. Units: milliseconds"),
		ECVF_Default);

	static TAutoConsoleVariable<float> CVarSyncTrackingTimeout(
		TEXT("AvaDisplayCluster.Sync.TrackingTimeout"), 5000.0f,
		TEXT("Delay after witch the tracked events are discarded. Units: milliseconds"),
		ECVF_Default);
}

FAvaDisplayClusterSynchronizedEventDispatcher::FAvaDisplayClusterSynchronizedEventDispatcher(const FString& InSignature)
	: Signature(InSignature)
{
	if (const IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr())
	{
		NodeId = ClusterManager->GetNodeId();
		ClusterManager->GetNodeIds(AllNodeIds);
	}
}

FString FAvaDisplayClusterSynchronizedEventDispatcher::GetFrameInfo() const
{
	return FString::Printf(TEXT("%s (disp:%s)"), *UE::AvaPlayback::Utils::GetBriefFrameInfo(), *Signature);
}

void FAvaDisplayClusterSynchronizedEventDispatcher::OnClusterEventReceived(const FAvaDisplayClusterClusterEventPayload& InPayload)
{
	using namespace UE::AvaDisplayCluster::Private;

	UE_LOGF(LogAvaDisplayClusterSyncEvents, Verbose, "%ls Received cluster event \"%ls\" (count:%d) from \"%ls\".",
		*GetFrameInfo(), *InPayload.Signature, InPayload.EmitCount, *InPayload.NodeId);

	// Todo: Check for re-emission (if necessary)
	//if (Payload.NodeId != NodeId && Payload.EmitCount > 1)
	//{
		// If it is either pending, ready or invoked, we should emit it again.
		// It means the other node didn't receive previous signal.
		// This will likely cause desyncs but will lead to faster recovery then waiting for a full timeout.
	//}

	// TODO: Investigate why this happens...
	if (ReadyEvents.Contains(InPayload.Signature))
	{
		UE_LOGF(LogAvaDisplayClusterSyncEvents, Warning, "%ls Event \"%ls\" is already ready on \"%ls\".",
			*GetFrameInfo(), *InPayload.Signature, *NodeId);
		
		if (PendingEvents.Contains(InPayload.Signature))
		{
			PendingEvents.Remove(InPayload.Signature);
			UE_LOGF(LogAvaDisplayClusterSyncEvents, Warning, "%ls Event \"%ls\" was pending on \"%ls\". Removing...",
				*GetFrameInfo(), *InPayload.Signature, *NodeId);
		}
		
		if (TrackedEvents.Contains(InPayload.Signature))
		{
			TrackedEvents.Remove(InPayload.Signature);
			UE_LOGF(LogAvaDisplayClusterSyncEvents, Warning, "%ls Event \"%ls\" was tracked on \"%ls\". Removing...",
				*GetFrameInfo(), *InPayload.Signature, *NodeId);
		}
		return;
	}

	if (TUniquePtr<FAvaDisplayClusterSynchronizedEvent>* FoundEvent = PendingEvents.Find(InPayload.Signature))
	{
		(*FoundEvent)->NodeInfo.Mark(InPayload.NodeId);

		if ((*FoundEvent)->NodeInfo.IsAllMarked(AllNodeIds))
		{
			UE_LOGF(LogAvaDisplayClusterSyncEvents, Verbose, "%ls Event \"%ls\" is ready on \"%ls\".", *GetFrameInfo(), *InPayload.Signature, *NodeId);

			if (CVarSyncEarlyDispatch.GetValueOnAnyThread())
			{
				DispatchEvent((*FoundEvent).Get(), FAvaDisplayClusterTimeStamp::Now());
			}
			else
			{
				ReadyEvents.Add(InPayload.Signature, MoveTemp(*FoundEvent));
			}
			
			PendingEvents.Remove(InPayload.Signature);
		}
	}
	else
	{
		// Tracking an event from the same node is considered an error state. It will just expire after a while.
		// If this happens, it is a symptom of another problem that will need to be investigated.
		// One cause is having a dispatch timeout shorter than the repeat timeout causing local re-emission to end up tracked.
		if (NodeId == InPayload.NodeId)
		{
			UE_LOGF(LogAvaDisplayClusterSyncEvents, Error, "%ls Tracking Event \"%ls\" on \"%ls\" (same node).",
				*GetFrameInfo(), *InPayload.Signature, *NodeId);
		}

		// Corresponding local event hasn't been pushed yet, we need to add a tracked event
		// to mark the nodeId. It will be transferred to the real event when it is locally pushed.
		if (const TUniquePtr<FAvaDisplayClusterTrackedClusterEvent>* FoundTrackedEvent = TrackedEvents.Find(InPayload.Signature))
		{
			(*FoundTrackedEvent)->NodeInfo.Mark(InPayload.NodeId);
			(*FoundTrackedEvent)->ReceivedTimeStamp = FAvaDisplayClusterTimeStamp::Now();

			UE_LOGF(LogAvaDisplayClusterSyncEvents, Verbose, "%ls Tracked Event \"%ls\" touched (on \"%ls\"): Marking \"%ls\".",
				*GetFrameInfo(), *InPayload.Signature, *NodeId, *InPayload.NodeId);
		}
		else
		{
			TUniquePtr<FAvaDisplayClusterTrackedClusterEvent> TrackedEvent = MakeUnique<FAvaDisplayClusterTrackedClusterEvent>();
			TrackedEvent->NodeInfo.Mark(InPayload.NodeId);
			TrackedEvent->ReceivedTimeStamp = FAvaDisplayClusterTimeStamp::Now();
			TrackedEvents.Add(InPayload.Signature, MoveTemp(TrackedEvent));
		
			UE_LOGF(LogAvaDisplayClusterSyncEvents, Verbose, "%ls Tracked Event \"%ls\" created (on \"%ls\"): Marking for \"%ls\".",
				*GetFrameInfo(), *InPayload.Signature, *NodeId, *InPayload.NodeId);
		}
	}
}

void FAvaDisplayClusterSynchronizedEventDispatcher::EmitClusterEvent(FAvaDisplayClusterSynchronizedEvent& InEvent, const FAvaDisplayClusterTimeStamp& InNow, IDisplayClusterClusterManager* InClusterManager)
{
	using namespace UE::AvaDisplayCluster::Private;
	if (InClusterManager)
	{
		++InEvent.EmitCount;

		FArrayWriter ArrayWriter;
		FAvaDisplayClusterClusterEventPayload::Serialize(ArrayWriter, Signature, InEvent.Signature, NodeId, InEvent.EmitCount);
			
		FDisplayClusterClusterEventBinary ClusterEvent;
		ClusterEvent.bIsSystemEvent = true;
		ClusterEvent.bShouldDiscardOnRepeat = false;	// Multiple events with this Id are emitted.
		ClusterEvent.EventId = SynchronizedEventsClusterEventId;
		ClusterEvent.EventData = MoveTemp(ArrayWriter);
		constexpr bool bEmitFromPrimaryOnly = false;	// All nodes emit this event.
		InClusterManager->EmitClusterEventBinary(ClusterEvent, bEmitFromPrimaryOnly);

		InEvent.LastEmitTimeStamp = InNow;

		UE_LOGF(LogAvaDisplayClusterSyncEvents, Verbose, "%ls Emitting cluster event \"%ls\" (count:%d) from \"%ls\".",
			*GetFrameInfo(), *InEvent.Signature, InEvent.EmitCount, *NodeId);
	}
}

void FAvaDisplayClusterSynchronizedEventDispatcher::DispatchEvent(const FAvaDisplayClusterSynchronizedEvent* InEvent, const FAvaDisplayClusterTimeStamp& InNow ) const
{
	check(IsInGameThread());
	using namespace UE::AvaDisplayCluster::Private;
	if (!InEvent)
	{
		return;
	}
		
	if (InEvent->Function)
	{
		UE_LOGF(LogAvaDisplayClusterSyncEvents, Verbose, "%ls Invoking Event \"%ls\" on \"%ls\", wait time: %.2f ms (%d frames).",
			*GetFrameInfo(), *InEvent->Signature, *NodeId, InEvent->PushTimeStamp.GetWaitTimeInMs(InNow), InEvent->PushTimeStamp.GetWaitTimeInFrames(InNow));
			
		InEvent->Function();
	}
	else
	{
		UE_LOGF(LogAvaDisplayClusterSyncEvents, Error, "%ls Unbound Event \"%ls\" on \"%ls\".", *GetFrameInfo(), *InEvent->Signature, *NodeId);
	}
}

bool FAvaDisplayClusterSynchronizedEventDispatcher::PushEvent(FString&& InEventSignature, TUniqueFunction<void()> InFunction)
{
	check(IsInGameThread());
	using namespace UE::AvaDisplayCluster::Private;

	TUniquePtr<FAvaDisplayClusterSynchronizedEvent> Event
		= MakeUnique<FAvaDisplayClusterSynchronizedEvent>(MoveTemp(InEventSignature), MoveTemp(InFunction));
		
	Event->PushTimeStamp = FAvaDisplayClusterTimeStamp::Now();

	IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
	if (ClusterManager && IDisplayCluster::Get().GetOperationMode() == EDisplayClusterOperationMode::Cluster)
	{
		if (PendingEvents.Contains(Event->Signature))
		{
			UE_LOGF(LogAvaDisplayClusterSyncEvents, Warning, "%ls Event \"%ls\" is already pushed on \"%ls\".", *GetFrameInfo(), *Event->Signature, *NodeId);
			return false; // Don't stomp events. If an event is pushed more than once, only keep one.
		}

		// Check if we have a tracked event, if so, transfer marked node ids.
		if (const TUniquePtr<FAvaDisplayClusterTrackedClusterEvent>* FoundEvent = TrackedEvents.Find(Event->Signature))
		{
			Event->NodeInfo.NodeIds = MoveTemp((*FoundEvent)->NodeInfo.NodeIds);
			TrackedEvents.Remove(Event->Signature);
		}

		const TUniquePtr<FAvaDisplayClusterSynchronizedEvent>& AddedEvent = PendingEvents.Add(Event->Signature, MoveTemp(Event));
		EmitClusterEvent(*AddedEvent, AddedEvent->PushTimeStamp, ClusterManager);
	}
	else
	{
		if (CVarSyncEarlyDispatch.GetValueOnAnyThread())
		{
			DispatchEvent(Event.Get(), Event->PushTimeStamp);	
		}
		else
		{
			if (ReadyEvents.Contains(Event->Signature))
			{
				UE_LOGF(LogAvaDisplayClusterSyncEvents, Warning, "%ls Event \"%ls\" is already \"ready\" on \"%ls\". Previous event will be discarded.", *GetFrameInfo(), *Event->Signature, *NodeId);
			}
			ReadyEvents.Add(Event->Signature, MoveTemp(Event));
		}
	}
	return true;
}

EAvaMediaSynchronizedEventState FAvaDisplayClusterSynchronizedEventDispatcher::GetEventState(const FString& InEventSignature) const
{
	if (PendingEvents.Contains(InEventSignature))
	{
		return EAvaMediaSynchronizedEventState::Pending;
	}
	if (ReadyEvents.Contains(InEventSignature))
	{
		return EAvaMediaSynchronizedEventState::Ready;
	}
	if (TrackedEvents.Contains(InEventSignature))
	{
		return EAvaMediaSynchronizedEventState::Tracked;
	}
	return EAvaMediaSynchronizedEventState::NotFound;
}

void FAvaDisplayClusterSynchronizedEventDispatcher::DispatchEvents()
{
	check(IsInGameThread());
	using namespace UE::AvaDisplayCluster::Private;
	const FAvaDisplayClusterTimeStamp Now = FAvaDisplayClusterTimeStamp::Now();

	IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
	
	// Refresh the list of nodes (only once a frame) in case it has changed.
	// There doesn't seem to be an event for this.
	if (ClusterManager)
	{
		ClusterManager->GetNodeIds(AllNodeIds);
	}

	const double RepeatTimeoutInMs = CVarSyncRepeatTimeout.GetValueOnAnyThread();
	const double DispatchTimeoutInMs = CVarSyncDispatchTimeout.GetValueOnAnyThread();

	for (TMap<FString, TUniquePtr<FAvaDisplayClusterSynchronizedEvent>>::TIterator EventIt(PendingEvents); EventIt; ++EventIt)
	{
		if (EventIt->Value->NodeInfo.IsAllMarked(AllNodeIds))
		{
			DispatchEvent(EventIt->Value.Get(), Now);
			EventIt.RemoveCurrent();
			continue;
		}

		// Locally dispatch pending events that timed out.
		if (EventIt->Value->PushTimeStamp.GetWaitTimeInMs(Now) > DispatchTimeoutInMs)
		{
			UE_LOGF(LogAvaDisplayClusterSyncEvents, Warning, "%ls Pending Event \"%ls\" has timed out on \"%ls\". Wait time: %.2f ms",
				*GetFrameInfo(), *EventIt->Value->Signature, *NodeId, EventIt->Value->PushTimeStamp.GetWaitTimeInMs(Now));

			DispatchEvent(EventIt->Value.Get(), Now);
			EventIt.RemoveCurrent();
			
			// Todo: Investigate idea. if an event times out, could keep track of the node that didn't fire and
			// "mute it" (don't wait for it anymore) until it returns.

			continue;
		}

		// Failsafe: periodically re-emit pending events in case cluster event didn't get through.
		// (Note: make sure to not re-emit a timed out event. time out test first above.)
		if (EventIt->Value->LastEmitTimeStamp.GetWaitTimeInMs(Now) > RepeatTimeoutInMs)
		{
			UE_LOGF(LogAvaDisplayClusterSyncEvents, Verbose, "%ls Re-emitting event \"%ls\" on \"%ls\".",
				*GetFrameInfo(), *EventIt->Value->Signature, *NodeId);

			EmitClusterEvent(*EventIt->Value, Now, ClusterManager);
		}
	}

	// Tracking timeout.
	const double TrackingTimeoutInMs = CVarSyncTrackingTimeout.GetValueOnAnyThread();
	
	for (TMap<FString, TUniquePtr<FAvaDisplayClusterTrackedClusterEvent>>::TIterator TrackedEventIt(TrackedEvents); TrackedEventIt; ++TrackedEventIt)
	{
		if (TrackedEventIt->Value->ReceivedTimeStamp.GetWaitTimeInMs(Now) > TrackingTimeoutInMs)
		{
			UE_LOGF(LogAvaDisplayClusterSyncEvents, Warning, "%ls Tracked Event \"%ls\" has timed out on \"%ls\". Wait time: %.2f ms",
				*GetFrameInfo(), *TrackedEventIt->Key, *NodeId, TrackedEventIt->Value->ReceivedTimeStamp.GetWaitTimeInMs(Now));

			TrackedEventIt.RemoveCurrent();
		}
	}

	for (const TPair<FString, TUniquePtr<FAvaDisplayClusterSynchronizedEvent>>& Event : ReadyEvents)
	{
		DispatchEvent(Event.Value.Get(), Now);
	}
	ReadyEvents.Reset();
}

