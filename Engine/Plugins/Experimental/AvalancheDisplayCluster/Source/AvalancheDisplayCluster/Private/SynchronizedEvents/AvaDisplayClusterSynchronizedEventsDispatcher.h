// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "ModularFeature/AvaMediaSynchronizedEvent.h"
#include "ModularFeature/IAvaMediaSynchronizedEventDispatcher.h"
#include "Utils/AvaDisplayClusterTimeStamp.h"

class IDisplayClusterClusterManager;

struct FAvaDisplayClusterClusterEventPayload
{
	/** Dispatcher Signature */
	FString Dispatcher;
	/** Event Signature */
	FString Signature;
	FString NodeId;
	int32 EmitCount = 0;
		
	static void Serialize(FArchive& InArchive, FString& InOutDispatcher, FString& InOutSignature, FString& InOutNodeId, int32& InOutEmitCount)
	{
		InArchive << InOutDispatcher;
		InArchive << InOutSignature;
		InArchive << InOutNodeId;
		InArchive << InOutEmitCount;
	}

	void Serialize(FArchive& InArchive)
	{
		Serialize(InArchive, Dispatcher, Signature, NodeId, EmitCount);
	}
};

struct FAvaDisplayClusterNodeInfo
{
	void Mark(const FString& InNodeId)
	{
		NodeIds.Add(InNodeId);
	}

	bool IsAllMarked(const TArray<FString>& InNodeIds) const
	{
		return NodeIds.Num() == InNodeIds.Num();
	}

	TSet<FString> NodeIds;
};

struct FAvaDisplayClusterTrackedClusterEvent
{
	FAvaDisplayClusterNodeInfo NodeInfo;
	FAvaDisplayClusterTimeStamp ReceivedTimeStamp;
};

struct FAvaDisplayClusterSynchronizedEvent : public FAvaMediaSynchronizedEvent
{
	FAvaDisplayClusterSynchronizedEvent(const FString& InSignature, TUniqueFunction<void()> InFunction)
		: FAvaMediaSynchronizedEvent(InSignature, MoveTemp(InFunction))
	{
	}

	FAvaDisplayClusterSynchronizedEvent(FString&& InSignature, TUniqueFunction<void()> InFunction)
		: FAvaMediaSynchronizedEvent(MoveTemp(InSignature), MoveTemp(InFunction))
	{
	}

	/** Tracks which node is marked. */
	FAvaDisplayClusterNodeInfo NodeInfo;

	/** Tracks time at which the event was pushed. */
	FAvaDisplayClusterTimeStamp PushTimeStamp;

	/** Tracks last time the event was emitted. */
	FAvaDisplayClusterTimeStamp LastEmitTimeStamp;

	/** Tracks number of times the event has been emitted. */
	int32 EmitCount = 0;
};

class FAvaDisplayClusterSynchronizedEventDispatcher : public IAvaMediaSynchronizedEventDispatcher
{
public:
	FAvaDisplayClusterSynchronizedEventDispatcher(const FString& InSignature);

	virtual ~FAvaDisplayClusterSynchronizedEventDispatcher() override = default;

	// Formatted frame info: time stamp and dispatcher's signature.
	FString GetFrameInfo() const;

	void OnClusterEventReceived(const FAvaDisplayClusterClusterEventPayload& InPayload);

	void EmitClusterEvent(FAvaDisplayClusterSynchronizedEvent& InEvent, const FAvaDisplayClusterTimeStamp& InNow, IDisplayClusterClusterManager* InClusterManager);

	void DispatchEvent(const FAvaDisplayClusterSynchronizedEvent* InEvent, const FAvaDisplayClusterTimeStamp& InNow ) const;
	
	//~ Begin IAvaMediaSynchronizedEventDispatcher
	virtual bool PushEvent(FString&& InEventSignature, TUniqueFunction<void()> InFunction) override;
	virtual EAvaMediaSynchronizedEventState GetEventState(const FString& InEventSignature) const override;
	virtual void DispatchEvents() override;
	//~ End IAvaMediaSynchronizedEventDispatcher 

	/** Dispatcher's signature. (used for debugging purpose atm) */
	FString Signature;
	
	/** Id of the current Cluster Node. */
	FString NodeId;

	/** All nodes in the cluster. */
	TArray<FString> AllNodeIds;

	/** Pending events waiting on other node's signal. */
	TMap<FString, TUniquePtr<FAvaDisplayClusterSynchronizedEvent>> PendingEvents;

	/** Tracked Events that are received as cluster event prior to getting submitted locally. */
	TMap<FString, TUniquePtr<FAvaDisplayClusterTrackedClusterEvent>> TrackedEvents;
	
	/** Events Ready to be locally dispatched. */
	TMap<FString, TUniquePtr<FAvaDisplayClusterSynchronizedEvent>> ReadyEvents;
	
	static constexpr int32 SynchronizedEventsClusterEventId = 0xABCDEF01;
};
