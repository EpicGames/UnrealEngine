// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDisplayClusterSynchronizedEventsFeature.h"

#include "AvaDisplayClusterSyncEventsLog.h"
#include "AvaDisplayClusterSynchronizedEventsDispatcher.h"
#include "Cluster/DisplayClusterClusterEvent.h"
//#include "Cluster/IDisplayClusterClusterManager.h"
#include "IDisplayCluster.h"
#include "IDisplayClusterCallbacks.h"
#include "Misc/CoreDelegates.h"
#include "Playback/AvaPlaybackUtils.h"
#include "Serialization/MemoryReader.h"

DEFINE_LOG_CATEGORY(LogAvaDisplayClusterSyncEvents);

#define LOCTEXT_NAMESPACE "AvaDisplayClusterSynchronizedEvent"

FAvaDisplayClusterSynchronizedEventsFeature::FAvaDisplayClusterSynchronizedEventsFeature()
{
	if (IDisplayCluster* DisplayCluster = FModuleManager::LoadModulePtr<IDisplayCluster>(IDisplayCluster::ModuleName))
	{
		DisplayCluster->GetCallbacks().OnDisplayClusterStartSession().AddRaw(this, &FAvaDisplayClusterSynchronizedEventsFeature::OnDisplayClusterStartSession);
	}
	FCoreDelegates::OnEndFrame.AddRaw(this, &FAvaDisplayClusterSynchronizedEventsFeature::OnEndFrame);
}

FAvaDisplayClusterSynchronizedEventsFeature::~FAvaDisplayClusterSynchronizedEventsFeature()
{
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	if (IDisplayCluster::IsAvailable())
	{
		IDisplayCluster::Get().GetCallbacks().OnDisplayClusterStartSession().RemoveAll(this);
		
		if (IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr())
		{
			if (BinaryListener.IsBound())
			{
				ClusterManager->RemoveClusterEventBinaryListener(BinaryListener);
			}
		}
	}
}

FName FAvaDisplayClusterSynchronizedEventsFeature::GetName() const
{
	static const FName ImplementationName(TEXT("DisplayClusterSync"));
	return ImplementationName;
}

FText FAvaDisplayClusterSynchronizedEventsFeature::GetDisplayName() const
{
	static const FText DisplayName = LOCTEXT("AvaDisplayClusterSyncDisplayName", "nDisplay Sync");
	return DisplayName;
}

FText FAvaDisplayClusterSynchronizedEventsFeature::GetDisplayDescription() const
{
	static const FText DisplayDescription = LOCTEXT("AvaDisplayClusterSyncDescription",
		 "Synchronize events over nDisplay using Cluster Events.");
	return DisplayDescription;
}

int32 FAvaDisplayClusterSynchronizedEventsFeature::GetPriority() const
{
	// It is likely cluster synchronization will take priority over other implementations.
	return GetDefaultPriority() + 10;
}

TSharedPtr<IAvaMediaSynchronizedEventDispatcher> FAvaDisplayClusterSynchronizedEventsFeature::CreateDispatcher(const FString& InSignature)
{
	TSharedPtr<FAvaDisplayClusterSynchronizedEventDispatcher> Dispatcher = MakeShared<FAvaDisplayClusterSynchronizedEventDispatcher>(InSignature);

	// Issue an error if the dispatcher already exists.
	const TWeakPtr<FAvaDisplayClusterSynchronizedEventDispatcher>* FoundDispatcherWeak = DispatchersWeak.Find(InSignature);
	if (FoundDispatcherWeak && FoundDispatcherWeak->IsValid())
	{
		UE_LOGF(LogAvaDisplayClusterSyncEvents, Error, "Dispatcher \"%ls\" already exist. Previous dispatcher will no longuer received events.", *InSignature);
	}
	
	// Keep track of our dispatchers for event routing.
	DispatchersWeak.Add(InSignature, Dispatcher.ToWeakPtr());

	// Carry over the tracked events prior to the dispatcher creation.
	//
	// Expectation is that the creation of the dispatchers may be out of sync on different nodes because it is
	// triggered by means that may have a few frames of delay. But we don't expect corresponding dispatchers
	// to be create with a large desync (of seconds), for now (tracker has a timeout of a few seconds).
	//
	const TSharedPtr<FAvaDisplayClusterSynchronizedEventDispatcher>* FoundTrackingDispatcher = TrackingDispatchers.Find(InSignature);
	if (FoundTrackingDispatcher && FoundTrackingDispatcher->IsValid())
	{
		Dispatcher->TrackedEvents = MoveTemp((*FoundTrackingDispatcher)->TrackedEvents);
	}
	TrackingDispatchers.Remove(InSignature);
	
	return Dispatcher;
}

void FAvaDisplayClusterSynchronizedEventsFeature::OnDisplayClusterStartSession()
{
	IDisplayClusterClusterManager* ClusterManager = IDisplayCluster::Get().GetClusterMgr();
	if (!ClusterManager)
	{
		UE_LOGF(LogAvaDisplayClusterSyncEvents, Error, "Display Cluster Manager is not available. Synchronized events will be disabled.");
		return;
	}
	
	BinaryListener = FOnClusterEventBinaryListener::CreateRaw(this, &FAvaDisplayClusterSynchronizedEventsFeature::OnBinaryClusterEventReceived);
	ClusterManager->AddClusterEventBinaryListener(BinaryListener);
}

void FAvaDisplayClusterSynchronizedEventsFeature::OnEndFrame()
{
	// Timeout tracked events - We may need to revisit this for edge cases (ex: long cluster init).
	for (const TPair<FString, TSharedPtr<FAvaDisplayClusterSynchronizedEventDispatcher>>& TrackingDispatcher : TrackingDispatchers)
	{
		TrackingDispatcher.Value->DispatchEvents();
	}
}

void FAvaDisplayClusterSynchronizedEventsFeature::OnBinaryClusterEventReceived(const FDisplayClusterClusterEventBinary& InClusterEvent)
{
	if (InClusterEvent.EventId != FAvaDisplayClusterSynchronizedEventDispatcher::SynchronizedEventsClusterEventId)
	{
		return;
	}
		
	FMemoryReader Reader(InClusterEvent.EventData);
	FAvaDisplayClusterClusterEventPayload Payload;
	Payload.Serialize(Reader);
	
	if (const TWeakPtr<FAvaDisplayClusterSynchronizedEventDispatcher>* FoundDispatcherWeak = DispatchersWeak.Find(Payload.Dispatcher))
	{
		if (const TSharedPtr<FAvaDisplayClusterSynchronizedEventDispatcher> FoundDispatcher = FoundDispatcherWeak->Pin())
		{
			FoundDispatcher->OnClusterEventReceived(Payload);
			return;	// Handled.
		}
		else
		{
			DispatchersWeak.Remove(Payload.Dispatcher);	// Remove stale map entries.
		}
	}

	// If no existing dispatcher has handled this event, we need to keep track of it in case dispatcher's creation is
	// out of sync between the different nodes.
	if (const TSharedPtr<FAvaDisplayClusterSynchronizedEventDispatcher>* FoundTrackingDispatcher = TrackingDispatchers.Find(Payload.Dispatcher))
	{
		if (*FoundTrackingDispatcher)
		{
			(*FoundTrackingDispatcher)->OnClusterEventReceived(Payload);
			return; // Handled.
		}
		else
		{
			UE_LOGF(LogAvaDisplayClusterSyncEvents, Warning, "Stale Tracking Dispatcher Found.");
			TrackingDispatchers.Remove(Payload.Dispatcher);	// Remove stale map entries.
		}
	}
	
	const TSharedPtr<FAvaDisplayClusterSynchronizedEventDispatcher> TrackingDispatcher = MakeShared<FAvaDisplayClusterSynchronizedEventDispatcher>(Payload.Signature);
	TrackingDispatchers.Add(Payload.Signature, TrackingDispatcher);

	UE_LOGF(LogAvaDisplayClusterSyncEvents, Verbose, "%ls Tracking Dispatcher \"%ls\" created on \"%ls\".",
		*UE::AvaPlayback::Utils::GetBriefFrameInfo(),  *TrackingDispatcher->Signature, *TrackingDispatcher->NodeId);
	
	TrackingDispatcher->OnClusterEventReceived(Payload);
}

#undef LOCTEXT_NAMESPACE