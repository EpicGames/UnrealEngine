// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Cluster/IDisplayClusterClusterManager.h"
#include "ModularFeature/IAvaMediaSynchronizedEventsFeature.h"

class FAvaDisplayClusterSynchronizedEventDispatcher;
struct FDisplayClusterClusterEventBinary;

class FAvaDisplayClusterSynchronizedEventsFeature : public IAvaMediaSynchronizedEventsFeature
{
public:
	FAvaDisplayClusterSynchronizedEventsFeature();
	virtual ~FAvaDisplayClusterSynchronizedEventsFeature() override;

	//~ Begin IAvaMediaSynchronizedEventsFeature
	virtual FName GetName() const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetDisplayDescription() const override;
	virtual int32 GetPriority() const override;
	virtual TSharedPtr<IAvaMediaSynchronizedEventDispatcher> CreateDispatcher(const FString& InSignature) override;
	//~ End IAvaMediaSynchronizedEventsFeature

private:
	// Event Handlers
	void OnDisplayClusterStartSession();
	void OnEndFrame();
	void OnBinaryClusterEventReceived(const FDisplayClusterClusterEventBinary& InClusterEvent);
	
private:
	/** Keep track of the created dispatchers for routing the cluster events. */
	TMap<FString, TWeakPtr<FAvaDisplayClusterSynchronizedEventDispatcher>> DispatchersWeak;

	/** Dispatchers only used for tracking cluster events when there is no active dispatcher created. */
	TMap<FString, TSharedPtr<FAvaDisplayClusterSynchronizedEventDispatcher>> TrackingDispatchers;

	FOnClusterEventBinaryListener BinaryListener;
};