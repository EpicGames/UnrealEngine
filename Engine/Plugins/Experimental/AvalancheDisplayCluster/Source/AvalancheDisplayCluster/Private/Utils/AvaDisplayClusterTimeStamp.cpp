// Copyright Epic Games, Inc. All Rights Reserved.

#include "AvaDisplayClusterTimeStamp.h"

#include "CoreGlobals.h"

FAvaDisplayClusterTimeStamp FAvaDisplayClusterTimeStamp::Now()
{
	return { FDateTime::Now(), GFrameNumber};
}
