// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DateTime.h"

/**
 * @brief TimeStamp utility for scheduling events, timeout, repeating, etc.
 */
struct FAvaDisplayClusterTimeStamp
{
	static FAvaDisplayClusterTimeStamp Now();

	double GetWaitTimeInMs(const FAvaDisplayClusterTimeStamp& InNow) const
	{
		const FTimespan wait = InNow.Time - Time;
		return wait.GetTotalMilliseconds();
	}

	uint32 GetWaitTimeInFrames(const FAvaDisplayClusterTimeStamp& InNow) const
	{
		return InNow.FrameNumber - FrameNumber;	
	}

	/** Local System time. Not synchronized. */
	FDateTime Time;
	
	/** (Tentative) Keep track of synchronized frame number (for logging and detecting desyncs). */
	uint32 FrameNumber = 0;
};
